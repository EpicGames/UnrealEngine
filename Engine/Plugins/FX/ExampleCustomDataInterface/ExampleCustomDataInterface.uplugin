{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Niagara Example Custom DataInterface",
	"Description": "This plugin contains C++ example content that shows how to write your own data interface for Niagara. \nCheck out the plugin folder Engine/Plugins/FX/ExampleCustomDataInterface for the source files.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ExampleCustomDataInterface",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		}
	]
}