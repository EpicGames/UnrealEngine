// Copyright Epic Games, Inc. All Rights Reserved.


// when using a template ush file, we need the _{ParameterName} appendix on global functions and parameters, because the template can be included multiple times for different data interfaces in a system.
float4 {ParameterName}_MousePosition;

void GetMousePosition_{ParameterName}(in bool In_Normalized, out float Out_PosX, out float Out_PosY)
{
	float4 DIPosition = {ParameterName}_MousePosition;
	if (In_Normalized)
	{
		Out_PosX = DIPosition.x / DIPosition.z;
		Out_PosY = DIPosition.y / DIPosition.w;
	}
	else
	{
		Out_PosX = DIPosition.x;
		Out_PosY = DIPosition.y;
	}
}
