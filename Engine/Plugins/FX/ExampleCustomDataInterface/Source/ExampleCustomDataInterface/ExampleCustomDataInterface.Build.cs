// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ExampleCustomDataInterface : ModuleRules
{
	public ExampleCustomDataInterface(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new[]
			{
				"Core"
			});
			
		
		PrivateDependencyModuleNames.AddRange(
			new[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"Projects",
				
				// Data interface dependencies
				"Niagara", "NiagaraCore", "VectorVM", "RenderCore", "RHI"
			});
		
		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
