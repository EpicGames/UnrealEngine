// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCustomDataInterface.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"
#include "Interfaces/IPluginManager.h"

#define LOCTEXT_NAMESPACE "FExampleCustomDataInterfaceModule"

void FExampleCustomDataInterfaceModule::StartupModule()
{
	// map the shader dir so we can use it in the data interface
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("ExampleCustomDataInterface"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/ExampleCustomDataInterface"), PluginShaderDir);
}

void FExampleCustomDataInterfaceModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FExampleCustomDataInterfaceModule, ExampleCustomDataInterface)
