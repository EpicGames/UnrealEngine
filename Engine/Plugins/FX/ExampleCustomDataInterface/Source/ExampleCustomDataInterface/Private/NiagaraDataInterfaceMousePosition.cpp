// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceMousePosition.h"

#if WITH_EDITORONLY_DATA
#include "LevelEditorViewport.h"
#else
#include "Engine/World.h"
#endif

#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "GameFramework/PlayerController.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceMousePosition"

const FName UNiagaraDataInterfaceMousePosition::GetMousePositionName(TEXT("GetMousePosition"));
static const TCHAR* MouseDITemplateShaderFile = TEXT("/Plugin/ExampleCustomDataInterface/Private/NiagaraDataInterfaceMousePosition.ush");

// the struct used to store our data interface data
struct FNDIMousePositionInstanceData
{
	FVector2f MousePos;
	FIntPoint ScreenSize;
};

// this proxy is used to safely copy data between game thread and render thread
struct FNDIMousePositionProxy : public FNiagaraDataInterfaceProxy
{
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNDIMousePositionInstanceData); }

	static void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* InDataFromGameThread, const FNiagaraSystemInstanceID& SystemInstance)
	{
		// initialize the render thread instance data into the pre-allocated memory
		FNDIMousePositionInstanceData* DataForRenderThread = new (InDataForRenderThread) FNDIMousePositionInstanceData();

		// we're just copying the game thread data, but the render thread data can be initialized to anything here and can be another struct entirely
		const FNDIMousePositionInstanceData* DataFromGameThread = static_cast<FNDIMousePositionInstanceData*>(InDataFromGameThread);
		*DataForRenderThread = *DataFromGameThread;
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
	{
		FNDIMousePositionInstanceData* InstanceDataFromGT = static_cast<FNDIMousePositionInstanceData*>(PerInstanceData);
		FNDIMousePositionInstanceData& InstanceData = SystemInstancesToInstanceData_RT.FindOrAdd(InstanceID);
		InstanceData = *InstanceDataFromGT;

		// we call the destructor here to clean up the GT data. Without this we could be leaking memory.
		InstanceDataFromGT->~FNDIMousePositionInstanceData();
	}

	TMap<FNiagaraSystemInstanceID, FNDIMousePositionInstanceData> SystemInstancesToInstanceData_RT;
};

// creates a new data object to store our position in.
// Don't keep transient data on the data interface object itself, only use per instance data!
bool UNiagaraDataInterfaceMousePosition::InitPerInstanceData(void* PerInstanceData,	FNiagaraSystemInstance* SystemInstance)
{
	FNDIMousePositionInstanceData* InstanceData = new (PerInstanceData) FNDIMousePositionInstanceData;
	InstanceData->MousePos = FVector2f::ZeroVector;
	return true;
}

// clean up RT instances
void UNiagaraDataInterfaceMousePosition::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIMousePositionInstanceData* InstanceData = static_cast<FNDIMousePositionInstanceData*>(PerInstanceData);
	InstanceData->~FNDIMousePositionInstanceData();

	ENQUEUE_RENDER_COMMAND(RemoveProxy)
	(
		[RT_Proxy=GetProxyAs<FNDIMousePositionProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToInstanceData_RT.Remove(InstanceID);
		}
	);
}

int32 UNiagaraDataInterfaceMousePosition::PerInstanceDataSize() const
{
	return sizeof(FNDIMousePositionInstanceData);
}

// This ticks on the game thread and lets us do work to initialize the instance data.
// If you need to do work on the gathered instance data after the simulation is done, use PerInstanceTickPostSimulate() instead. 
bool UNiagaraDataInterfaceMousePosition::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance,	float DeltaSeconds)
{
	check(SystemInstance);
	FNDIMousePositionInstanceData* InstanceData = static_cast<FNDIMousePositionInstanceData*>(PerInstanceData);
	if (!InstanceData)
	{
		return true;
	}

	InstanceData->MousePos = FVector2f::ZeroVector;

	// If we have a player controller we use it to capture the mouse position
	UWorld* World = SystemInstance->GetWorld();
	if (World && World->GetNumPlayerControllers() > 0)
	{
		APlayerController* Controller = World->GetFirstPlayerController();
		Controller->GetMousePosition(InstanceData->MousePos.X, InstanceData->MousePos.Y);
		Controller->GetViewportSize(InstanceData->ScreenSize.X, InstanceData->ScreenSize.Y);
		return false;
	}
	
#if WITH_EDITORONLY_DATA
	// While in the editor we don't necessarily have a player controller, so we query the viewport object instead 
	if (GCurrentLevelEditingViewportClient)
	{
		InstanceData->MousePos.X = GCurrentLevelEditingViewportClient->Viewport->GetMouseX();
		InstanceData->MousePos.Y = GCurrentLevelEditingViewportClient->Viewport->GetMouseY();
		InstanceData->ScreenSize = GCurrentLevelEditingViewportClient->Viewport->GetSizeXY();
	}
#endif

	return false;
}

void UNiagaraDataInterfaceMousePosition::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNDIMousePositionProxy::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

UNiagaraDataInterfaceMousePosition::UNiagaraDataInterfaceMousePosition(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNDIMousePositionProxy());
}

// this registers our custom DI with Niagara
void UNiagaraDataInterfaceMousePosition::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
// this lists all the functions our DI provides (currently only one)
void UNiagaraDataInterfaceMousePosition::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = GetMousePositionName;
	Sig.Description = LOCTEXT("GetMousePositionNameFunctionDescription", "Returns the mouse position in screen space.");
	Sig.bMemberFunction = true;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("MousePosition interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Normalized")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("PosX")), LOCTEXT("MousePosXDescription", "Returns the x coordinates in pixels or 0-1 range if normalized"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("PosY")), LOCTEXT("MousePosYDescription", "Returns the y coordinates in pixels or 0-1 range if normalized"));
	OutFunctions.Add(Sig);
}
#endif

// this provides the cpu vm with the correct function to call
void UNiagaraDataInterfaceMousePosition::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == GetMousePositionName)
	{
		OutFunc = FVMExternalFunction::CreateLambda([this](FVectorVMExternalFunctionContext& Context) { this->GetMousePositionVM(Context); });
	}
	else
	{
		UE_LOGF(LogTemp, Display, "Could not find data interface external function in %ls. Received Name: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString());
	}
}

// implementation called by the vectorVM
void UNiagaraDataInterfaceMousePosition::GetMousePositionVM(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIMousePositionInstanceData> InstData(Context);
	FNDIInputParam<bool> InNormalized(Context);
	FNDIOutputParam<float> OutPosX(Context);
	FNDIOutputParam<float> OutPosY(Context);

	FVector2f MousePos = InstData.Get()->MousePos;
	FIntPoint ScreenSize = InstData.Get()->ScreenSize;

	// iterate over the particles
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		if (InNormalized.GetAndAdvance())
		{
			OutPosX.SetAndAdvance(MousePos.X / ScreenSize.X);
			OutPosY.SetAndAdvance(MousePos.Y / ScreenSize.Y);
		}
		else
		{
			OutPosX.SetAndAdvance(MousePos.X);
			OutPosY.SetAndAdvance(MousePos.Y);
		}
	}
}

#if WITH_EDITORONLY_DATA

// this lets the niagara compiler know that it needs to recompile an effect when our hlsl file changes
bool UNiagaraDataInterfaceMousePosition::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateShaderFile(MouseDITemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return true;
}

// This can be used to provide the hlsl code for gpu scripts. If the DI supports only cpu implementations, this is not needed.
// We don't need to actually print our function code to OutHLSL here because we use a template file that gets appended in GetParameterDefinitionHLSL().
// If the hlsl function is so simple that it does not need bound shader parameters, then this method can be used instead of GetParameterDefinitionHLSL.
bool UNiagaraDataInterfaceMousePosition::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	return FunctionInfo.DefinitionName == GetMousePositionName;
}

// this loads our hlsl template script file and 
void UNiagaraDataInterfaceMousePosition::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo,	FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, MouseDITemplateShaderFile, TemplateArgs);
}

#endif

// This fills in the expected parameter bindings we use to send data to the GPU
void UNiagaraDataInterfaceMousePosition::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

// This fills in the parameters to send to the GPU
void UNiagaraDataInterfaceMousePosition::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNDIMousePositionProxy& DataInterfaceProxy = Context.GetProxy<FNDIMousePositionProxy>();
	FNDIMousePositionInstanceData& InstanceData = DataInterfaceProxy.SystemInstancesToInstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->MousePosition.X = InstanceData.MousePos.X;
	ShaderParameters->MousePosition.Y = InstanceData.MousePos.Y;
	ShaderParameters->MousePosition.Z = InstanceData.ScreenSize.X;
	ShaderParameters->MousePosition.W = InstanceData.ScreenSize.Y;
}

#undef LOCTEXT_NAMESPACE
