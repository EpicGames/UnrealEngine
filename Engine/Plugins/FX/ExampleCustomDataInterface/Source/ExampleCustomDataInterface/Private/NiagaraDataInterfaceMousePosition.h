// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceMousePosition.generated.h"

enum class ENiagaraSimTarget : uint8;
struct FNiagaraDataInterfaceGeneratedFunction;
struct FNiagaraFunctionSignature;
struct FVMExternalFunctionBindingInfo;

UCLASS(EditInlineNew, Category = "Mouse", meta = (DisplayName = "MousePosition Query"))
class UNiagaraDataInterfaceMousePosition : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector4f,				MousePosition)
	END_SHADER_PARAMETER_STRUCT()

public:
	//UObject Interface
	virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
#if WITH_EDITORONLY_DATA
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual int32 PerInstanceDataSize() const override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
	//UNiagaraDataInterface Interface

	void GetMousePositionVM(FVectorVMExternalFunctionContext& Context);	

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

private:
	static const FName GetMousePositionName;
};
